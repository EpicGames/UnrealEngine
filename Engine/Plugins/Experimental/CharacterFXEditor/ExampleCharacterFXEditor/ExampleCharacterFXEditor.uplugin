{
	"FileVersion": 1,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "ExampleCharacterFXEditor",
	"Description": "Example asset editor using the BaseCharacterFXEditor base classes",
	"Category": "Other",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "ExampleCharacterFXEditor",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [ "Win64" ]
		}
	],
	"Plugins": [
		{
			"Name": "BaseCharacterFXEditor",
			"Enabled": true
		},
		{
			"Name": "MeshModelingToolset",
			"Enabled": true
		},
		{
			"Name": "MeshModelingToolsetExp",
			"Enabled": true
		}
	]
}