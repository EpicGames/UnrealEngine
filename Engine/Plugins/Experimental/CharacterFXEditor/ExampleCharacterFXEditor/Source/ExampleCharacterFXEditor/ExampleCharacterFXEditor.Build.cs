// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ExampleCharacterFXEditor : ModuleRules
{
	public ExampleCharacterFXEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		// Make the linking path shorter (to prevent breaking the Windows limit)
		ShortName = "ExDefEd";

		PublicIncludePaths.AddRange(
			new string[] {
				// ... add public include paths required here ...
			}
			);
				
		
		PrivateIncludePaths.AddRange(
			new string[] {
				// ... add other private include paths required here ...
			}
			);
			
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{			
				// ... add other public dependencies that you statically link with here ...
			}
			);
			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"BaseCharacterFXEditor",
				"Core",
				"CoreUObject",
				"ContentBrowser",
				"DynamicMesh",
				"EditorFramework",
				"EditorInteractiveToolsFramework",
				"EditorStyle",
				"EditorSubsystem",
				"Engine",
				"GeometryCore",
				"GeometryFramework",
				"InputCore",
				"InteractiveToolsFramework",
				"MeshModelingToolsEditorOnly",
				"MeshModelingToolsEditorOnlyExp",
				"ModelingComponentsEditorOnly",
				"ModelingComponents",
				"Projects",
				"Slate",
				"SlateCore",
				"ToolMenus",
				"UnrealEd",
				// ... add private dependencies that you statically link with here ...	
			}
			);
		
		
		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				// ... add any modules that your module loads dynamically here ...
			}
			);
	}
}
