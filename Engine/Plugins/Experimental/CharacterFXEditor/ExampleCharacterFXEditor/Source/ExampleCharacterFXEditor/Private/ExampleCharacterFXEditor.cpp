// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExampleCharacterFXEditor.h"
#include "ExampleCharacterFXEditorToolkit.h"

TSharedPtr<FBaseAssetToolkit> UExampleCharacterFXEditor::CreateToolkit()
{
	TSharedPtr<FExampleCharacterFXEditorToolkit> Toolkit = MakeShared<FExampleCharacterFXEditorToolkit>(this);
	return Toolkit;
}
