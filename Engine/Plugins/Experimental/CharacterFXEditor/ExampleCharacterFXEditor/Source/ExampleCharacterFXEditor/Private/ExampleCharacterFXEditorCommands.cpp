// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExampleCharacterFXEditorCommands.h"
#include "ExampleCharacterFXEditorStyle.h"

#define LOCTEXT_NAMESPACE "FExampleCharacterFXEditorCommands"

const FString FExampleCharacterFXEditorCommands::BeginAttributeEditorToolIdentifier = TEXT("BeginAttributeEditorTool");

FExampleCharacterFXEditorCommands::FExampleCharacterFXEditorCommands()
	: TBaseCharacterFXEditorCommands<FExampleCharacterFXEditorCommands>("ExampleCharacterFXEditor",
		LOCTEXT("ContextDescription", "Example CharacterFX Editor"), 
		NAME_None, // Parent
		FExampleCharacterFXEditorStyle::StyleName)
{
}

void FExampleCharacterFXEditorCommands::RegisterCommands()
{
	TBaseCharacterFXEditorCommands::RegisterCommands();

	UI_COMMAND(OpenCharacterFXEditor, "CharacterFX Editor", "Open the CharacterFX Editor window", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(BeginAttributeEditorTool, "AttrEd", "Edit/configure mesh attributes", EUserInterfaceActionType::Button, FInputChord());
}


#undef LOCTEXT_NAMESPACE
