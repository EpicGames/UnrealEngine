// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExampleCharacterFXEditorMode.h"
#include "ExampleCharacterFXEditorCommands.h"
#include "ExampleCharacterFXEditorModeToolkit.h"
#include "ExampleCharacterFXEditorSubsystem.h"
#include "AttributeEditorTool.h"
#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/DynamicMeshCommitter.h"
#include "TargetInterfaces/DynamicMeshProvider.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "ToolTargetManager.h"
#include "ToolTargets/StaticMeshComponentToolTarget.h"
#include "ToolTargets/DynamicMeshComponentToolTarget.h"
#include "ToolTargets/SkeletalMeshComponentToolTarget.h"
#include "PreviewScene.h"
#include "AssetEditorModeManager.h"
#include "Engine/Selection.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "ToolTargets/ToolTarget.h"
#include "ModelingToolTargetUtil.h"
#include "Components/DynamicMeshComponent.h"

#define LOCTEXT_NAMESPACE "UExampleCharacterFXEditorMode"

const FEditorModeID UExampleCharacterFXEditorMode::EM_ExampleCharacterFXEditorModeId = TEXT("EM_ExampleCharacterFXEditorModeId");

UExampleCharacterFXEditorMode::UExampleCharacterFXEditorMode()
{
	Info = FEditorModeInfo(
		EM_ExampleCharacterFXEditorModeId,
		LOCTEXT("ExampleCharacterFXEditorModeName", "ExampleCharacterFXEditor"),
		FSlateIcon(),
		false);
}


void UExampleCharacterFXEditorMode::Exit()
{
	UBaseCharacterFXEditorMode::Exit();
	DynamicMeshComponents.Reset();
}

const FToolTargetTypeRequirements& UExampleCharacterFXEditorMode::GetToolTargetRequirements()
{
	static const FToolTargetTypeRequirements ToolTargetRequirements =
		FToolTargetTypeRequirements({
			UMaterialProvider::StaticClass(),
			UDynamicMeshCommitter::StaticClass(),
			UDynamicMeshProvider::StaticClass()
			});

	return ToolTargetRequirements;
}


void UExampleCharacterFXEditorMode::AddToolTargetFactories()
{
	GetInteractiveToolsContext()->TargetManager->AddTargetFactory(NewObject<UStaticMeshComponentToolTargetFactory>(GetToolManager()));
	GetInteractiveToolsContext()->TargetManager->AddTargetFactory(NewObject<UDynamicMeshComponentToolTargetFactory>(GetToolManager()));
	GetInteractiveToolsContext()->TargetManager->AddTargetFactory(NewObject<USkeletalMeshComponentToolTargetFactory>(GetToolManager()));
}

void UExampleCharacterFXEditorMode::RegisterTools()
{
	const FExampleCharacterFXEditorCommands& CommandInfos = FExampleCharacterFXEditorCommands::Get();

	RegisterTool(CommandInfos.BeginAttributeEditorTool, FExampleCharacterFXEditorCommands::BeginAttributeEditorToolIdentifier, NewObject<UAttributeEditorToolBuilder>());
}

void UExampleCharacterFXEditorMode::CreateToolkit()
{
	Toolkit = MakeShared<FExampleCharacterFXEditorModeToolkit>();
}

void UExampleCharacterFXEditorMode::BindCommands()
{
	const FExampleCharacterFXEditorCommands& CommandInfos = FExampleCharacterFXEditorCommands::Get();
	const TSharedRef<FUICommandList>& CommandList = Toolkit->GetToolkitCommands();

	// Hook up to Enter/Esc key presses
	CommandList->MapAction(
		CommandInfos.AcceptOrCompleteActiveTool,
		FExecuteAction::CreateLambda([this]() { AcceptActiveToolActionOrTool(); }),
		FCanExecuteAction::CreateLambda([this]() {
		return GetInteractiveToolsContext()->CanAcceptActiveTool() || GetInteractiveToolsContext()->CanCompleteActiveTool();
	}),
		FGetActionCheckState(),
		FIsActionButtonVisible(),
		EUIActionRepeatMode::RepeatDisabled);

	CommandList->MapAction(
		CommandInfos.CancelOrCompleteActiveTool,
		FExecuteAction::CreateLambda([this]() { CancelActiveToolActionOrTool(); }),
		FCanExecuteAction::CreateLambda([this]() {
		return GetInteractiveToolsContext()->CanCompleteActiveTool() || GetInteractiveToolsContext()->CanCancelActiveTool();
	}),
		FGetActionCheckState(),
		FIsActionButtonVisible(),
		EUIActionRepeatMode::RepeatDisabled);
}

void UExampleCharacterFXEditorMode::CreateToolTargets(const TArray<TObjectPtr<UObject>>& AssetsIn)
{
	UExampleCharacterFXEditorSubsystem* Subsystem = GEditor->GetEditorSubsystem<UExampleCharacterFXEditorSubsystem>();
	Subsystem->BuildTargets(AssetsIn, GetToolTargetRequirements(), ToolTargets);
}

void UExampleCharacterFXEditorMode::InitializeTargets(const TArray<TObjectPtr<UObject>>& AssetsIn)
{
	UBaseCharacterFXEditorMode::InitializeTargets(AssetsIn);
	
	USelection* SelectedComponents = GetModeManager()->GetSelectedComponents();
	SelectedComponents->Modify();
	SelectedComponents->BeginBatchSelectOperation();
	SelectedComponents->DeselectAll();

	// NOTE: Since the Example CharacterFX Editor is not the "default" editor for a particular asset type and we still want to have something to render,
	// we instead store a set of temporary DynamicMeshComponents here. This should not be necessary if your editor is the default for an asset type.
	for (UToolTarget* Target : ToolTargets)
	{
		UE::Geometry::FDynamicMesh3 DynamicMesh = UE::ToolTarget::GetDynamicMeshCopy(Target);

		TObjectPtr<UDynamicMeshComponent> DynamicMeshComponent = NewObject<UDynamicMeshComponent>();
		
		DynamicMeshComponent->SetMesh(MoveTemp(DynamicMesh));
		FComponentMaterialSet TargetMaterialSet = UE::ToolTarget::GetMaterialSet(Target);
		DynamicMeshComponent->ConfigureMaterialSet(TargetMaterialSet.Materials);
		check(DynamicMeshComponent->ValidateMaterialSlots(false, false));

		DynamicMeshComponent->RegisterComponentWithWorld(GetWorld());
		
		DynamicMeshComponents.Emplace(DynamicMeshComponent);
		SelectedComponents->Select(DynamicMeshComponent);
	}

	SelectedComponents->EndBatchSelectOperation();
}


FBox UExampleCharacterFXEditorMode::SceneBoundingBox() const
{
	FBoxSphereBounds TotalBounds;
	bool bAnyValid = false;

	for (const TObjectPtr<UDynamicMeshComponent>& MeshComponent : DynamicMeshComponents)
	{
		if (MeshComponent)
		{
			if (!bAnyValid)
			{
				TotalBounds = MeshComponent->Bounds;
				bAnyValid = true;
			}
			else
			{
				TotalBounds = TotalBounds + MeshComponent->Bounds;
			}
		}
	}

	if (bAnyValid)
	{
		return TotalBounds.GetBox();
	}
	else
	{
		return FBox(ForceInitToZero);
	}
}

#undef LOCTEXT_NAMESPACE
