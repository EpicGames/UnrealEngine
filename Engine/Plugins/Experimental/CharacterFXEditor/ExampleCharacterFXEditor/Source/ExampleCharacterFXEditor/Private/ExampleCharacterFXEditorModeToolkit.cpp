// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExampleCharacterFXEditorModeToolkit.h"
#include "ExampleCharacterFXEditorCommands.h"
#include "ExampleCharacterFXEditorStyle.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#define LOCTEXT_NAMESPACE "FExampleCharacterFXEditorModeToolkit"

FName FExampleCharacterFXEditorModeToolkit::GetToolkitFName() const
{
	return FName("ExampleCharacterFXEditorModeToolkit");
}

FText FExampleCharacterFXEditorModeToolkit::GetBaseToolkitName() const
{
	return NSLOCTEXT("ExampleCharacterFXEditorModeToolkit", "DisplayName", "ExampleCharacterFXEditorModeToolkit");
}

const FSlateBrush* FExampleCharacterFXEditorModeToolkit::GetActiveToolIcon(const FString& ActiveToolIdentifier) const
{
	FName ActiveToolIconName = ISlateStyle::Join(FExampleCharacterFXEditorCommands::Get().GetContextName(), TCHAR_TO_ANSI(*ActiveToolIdentifier));
	return FExampleCharacterFXEditorStyle::Get().GetOptionalBrush(ActiveToolIconName);
}

void FExampleCharacterFXEditorModeToolkit::BuildToolPalette(FName PaletteIndex, class FToolBarBuilder& ToolbarBuilder)
{
	const FExampleCharacterFXEditorCommands& Commands = FExampleCharacterFXEditorCommands::Get();

	if (PaletteIndex == ToolsTabName)
	{
		ToolbarBuilder.AddToolBarButton(Commands.BeginAttributeEditorTool);
	}
}

#undef LOCTEXT_NAMESPACE
