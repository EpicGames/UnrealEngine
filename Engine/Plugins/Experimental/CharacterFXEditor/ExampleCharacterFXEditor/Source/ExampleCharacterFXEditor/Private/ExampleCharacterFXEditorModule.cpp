// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExampleCharacterFXEditorModule.h"
#include "Editor.h"
#include "ExampleCharacterFXEditorMode.h"
#include "ExampleCharacterFXEditorStyle.h"
#include "ExampleCharacterFXEditorCommands.h"
#include "ExampleCharacterFXEditorSubsystem.h"
#include "ToolMenus.h"
#include "EditorModeRegistry.h"
#include "ContentBrowserMenuContexts.h"

#define LOCTEXT_NAMESPACE "FExampleCharacterFXEditorModule"

void FExampleCharacterFXEditorModule::StartupModule()
{
	FExampleCharacterFXEditorStyle::Get(); // Causes the constructor to be called
	FExampleCharacterFXEditorCommands::Register();

	// Menus need to be registered in a callback to make sure the system is ready for them.
	UToolMenus::RegisterStartupCallback(
		FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FExampleCharacterFXEditorModule::RegisterMenus));

	// TODO: Register details view customizations
}

void FExampleCharacterFXEditorModule::ShutdownModule()
{
	// Clean up menu things
	UToolMenus::UnRegisterStartupCallback(this);
	UToolMenus::UnregisterOwner(this);

	FExampleCharacterFXEditorCommands::Unregister();

	FEditorModeRegistry::Get().UnregisterMode(UExampleCharacterFXEditorMode::EM_ExampleCharacterFXEditorModeId);

	// TODO: Unregister details view customizations
}


//
// NOTE: Only necessary because we want to allow opening specific asset types from the content browser
//
void FExampleCharacterFXEditorModule::RegisterMenus()
{
	// Allows cleanup when module unloads.
	FToolMenuOwnerScoped OwnerScoped(this);

	// Extend the content browser context menu for static meshes and skeletal meshes
	auto AddToContextMenuSection = [this](FToolMenuSection& Section)
	{
		Section.AddDynamicEntry("OpenCharacterFXEditor", FNewToolMenuSectionDelegate::CreateLambda(
			[](FToolMenuSection& Section)
			{
				// We'll need to get the target objects out of the context
				if (UContentBrowserAssetContextMenuContext* Context = Section.FindContext<UContentBrowserAssetContextMenuContext>())
				{
					UExampleCharacterFXEditorSubsystem* ExampleCharacterFXEditorSubsystem = GEditor->GetEditorSubsystem<UExampleCharacterFXEditorSubsystem>();
					check(ExampleCharacterFXEditorSubsystem);

					// We are deliberately not using Context->GetSelectedObjects() here to avoid triggering a load from right clicking
					// an asset in the content browser.
					if (ExampleCharacterFXEditorSubsystem->AreAssetsValidTargets(Context->SelectedAssets))
					{
						TSharedPtr<class FUICommandList> CommandListToBind = MakeShared<FUICommandList>();

						CommandListToBind->MapAction(
							FExampleCharacterFXEditorCommands::Get().OpenCharacterFXEditor,
							FExecuteAction::CreateWeakLambda(ExampleCharacterFXEditorSubsystem, [Context, ExampleCharacterFXEditorSubsystem]()
							{
								// When we actually do want to open the editor, trigger the load to get the objects
								TArray<TObjectPtr<UObject>> AssetsToEdit;
								AssetsToEdit.Append(Context->LoadSelectedObjects<UObject>());

								// If we fail the ensure here, then there must be something that we're failing to check properly
								// in AreAssetsValidTargets that we would need to track down and check in the asset data.
								if (ensure(ExampleCharacterFXEditorSubsystem->AreObjectsValidTargets(AssetsToEdit)))
								{
									ExampleCharacterFXEditorSubsystem->StartExampleCharacterFXEditor(AssetsToEdit);
								}
							}));

						Section.AddMenuEntryWithCommandList(FExampleCharacterFXEditorCommands::Get().OpenCharacterFXEditor,
							CommandListToBind, 
							TAttribute<FText>(), 
							TAttribute<FText>(), 
							FSlateIcon(FAppStyle::GetAppStyleSetName(), "ExampleCharacterFXEditor.OpenCharacterFXEditor"));
					}
				}
			}));
	};
	{
		UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("ContentBrowser.AssetContextMenu.StaticMesh");
		FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
		AddToContextMenuSection(Section);
	}
	{
		UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("ContentBrowser.AssetContextMenu.SkeletalMesh");
		FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
		AddToContextMenuSection(Section);
	}
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FExampleCharacterFXEditorModule, ExampleCharacterFXEditor)
