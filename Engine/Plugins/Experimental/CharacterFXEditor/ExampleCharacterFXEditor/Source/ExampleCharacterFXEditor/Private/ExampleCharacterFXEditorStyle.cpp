// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExampleCharacterFXEditorStyle.h"
#include "ExampleCharacterFXEditorCommands.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "EditorStyleSet.h"
#include "Styling/SlateStyleMacros.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"

const FName FExampleCharacterFXEditorStyle::StyleName("ExampleCharacterFXEditorStyle");

FString FExampleCharacterFXEditorStyle::InContent(const FString& RelativePath, const ANSICHAR* Extension)
{
	static FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("ExampleCharacterFXEditor"))->GetContentDir();
	FString Path = (ContentDir / RelativePath) + Extension;
	return Path;
}

FExampleCharacterFXEditorStyle::FExampleCharacterFXEditorStyle()
	: FSlateStyleSet(StyleName)
{
	// Used FUVEditorStyle and FModelingToolsEditorModeStyle and as models

	// Some standard icon sizes used elsewhere in the editor
	const FVector2D Icon8x8(8.0f, 8.0f);
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon20x20(20.0f, 20.0f);
	const FVector2D Icon28x28(28.0f, 28.0f);
	const FVector2D Icon40x40(40.0f, 40.0f);
	const FVector2D Icon120(120.0f, 120.0f);

	// Icon sizes used in this style set
	const FVector2D ViewportToolbarIconSize = Icon16x16;
	const FVector2D ToolbarIconSize = Icon20x20;

	FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("ExampleCharacterFXEditor"))->GetContentDir();
	SetContentRoot(ContentDir);
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	FString PropertyNameString = "ExampleCharacterFXEditor." + FExampleCharacterFXEditorCommands::BeginAttributeEditorToolIdentifier;
	Set(*PropertyNameString, new FSlateImageBrush(FExampleCharacterFXEditorStyle::InContent("Icons/AttributeEditor_40x", ".png"), ToolbarIconSize));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FExampleCharacterFXEditorStyle::~FExampleCharacterFXEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}

FExampleCharacterFXEditorStyle& FExampleCharacterFXEditorStyle::Get()
{
	static FExampleCharacterFXEditorStyle Inst;
	return Inst;
}


