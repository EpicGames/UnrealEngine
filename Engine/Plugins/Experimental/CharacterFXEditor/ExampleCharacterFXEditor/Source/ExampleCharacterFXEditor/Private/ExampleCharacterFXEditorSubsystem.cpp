// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExampleCharacterFXEditorSubsystem.h"
#include "ExampleCharacterFXEditor.h"
#include "ExampleCharacterFXEditorMode.h"
#include "ToolTargets/DynamicMeshComponentToolTarget.h"
#include "ToolTargets/StaticMeshToolTarget.h"
#include "ToolTargets/SkeletalMeshToolTarget.h"
#include "ToolTargetManager.h"

using namespace UE::Geometry;

void UExampleCharacterFXEditorSubsystem::CreateToolTargetFactories(TArray<TObjectPtr<UToolTargetFactory>>& Factories) const
{
	Factories.Add(NewObject<UStaticMeshToolTargetFactory>(ToolTargetManager));
	Factories.Add(NewObject<USkeletalMeshToolTargetFactory>(ToolTargetManager));
	Factories.Add(NewObject<UDynamicMeshComponentToolTargetFactory>(ToolTargetManager));
}

void UExampleCharacterFXEditorSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	// The subsystem has its own tool target manager because it must exist before any ExampleCharacterFXEditors exist,
	// to see if the editor can be started.
	ToolTargetManager = NewObject<UToolTargetManager>(this);
	ToolTargetManager->Initialize();

	TArray<TObjectPtr<UToolTargetFactory>> ToolTargetFactories;
	CreateToolTargetFactories(ToolTargetFactories);

	for (const TObjectPtr<UToolTargetFactory>& Factory : ToolTargetFactories)
	{
		ToolTargetManager->AddTargetFactory(Factory);
	}
}

void UExampleCharacterFXEditorSubsystem::Deinitialize()
{
	ToolTargetManager->Shutdown();
	ToolTargetManager = nullptr;
}

void UExampleCharacterFXEditorSubsystem::BuildTargets(const TArray<TObjectPtr<UObject>>& ObjectsIn,
	const FToolTargetTypeRequirements& TargetRequirements,
	TArray<TObjectPtr<UToolTarget>>& TargetsOut)
{
	TargetsOut.Reset();

	for (UObject* Object : ObjectsIn)
	{
		UToolTarget* Target = ToolTargetManager->BuildTarget(Object, TargetRequirements);
		if (Target)
		{
			TargetsOut.Add(Target);
		}
	}
}

bool UExampleCharacterFXEditorSubsystem::AreObjectsValidTargets(const TArray<UObject*>& InObjects) const
{
	if (InObjects.IsEmpty())
	{
		return false;
	}

	for (UObject* Object : InObjects)
	{
		if (!ToolTargetManager->CanBuildTarget(Object, UExampleCharacterFXEditorMode::GetToolTargetRequirements()))
		{
			return false;
		}
	}

	return true;
}

bool UExampleCharacterFXEditorSubsystem::AreAssetsValidTargets(const TArray<FAssetData>& InAssets) const
{
	if (InAssets.IsEmpty())
	{
		return false;
	}

	// Currently our tool target factories don't evaluate FAssetData to figure out whether they can
	// build a tool target (they only work on UObjects directly), so for now we do corresponding checks
	// here ourselves according to the tooltargets that we support.
	auto IsValidStaticMeshAsset = [](const FAssetData& AssetData)
	{
		// The static mesh tool target checks GetNumSourceModels, which we can't do directly, hence our check of the LODs tag
		int32 NumLODs = 0;
		return AssetData.IsInstanceOf<UStaticMesh>() && AssetData.GetTagValue<int32>("LODs", NumLODs) && NumLODs > 0;
	};
	auto IsValidSkeletalMeshAsset = [](const FAssetData& AssetData)
	{
		// The skeletal mesh tool targets don't seem to try to check the number of LODs, but the skeletal mesh tool target
		// uses an exact cast for the class, hence the '==' comparison here.
		return AssetData.GetClass() == USkeletalMesh::StaticClass();
	};

	for (const FAssetData& AssetData : InAssets)
	{
		if (!IsValidStaticMeshAsset(AssetData) && !IsValidSkeletalMeshAsset(AssetData))
		{
			return false;
		}
	}

	return true;
}

void UExampleCharacterFXEditorSubsystem::StartExampleCharacterFXEditor(TArray<TObjectPtr<UObject>> ObjectsToEdit)
{
	// We don't allow opening a new instance if any of the objects are already opened
	// in an existing instance. Instead, we bring such an instance to the front.
	// Note that the asset editor subsystem takes care of this for "primary" asset editors, 
	// i.e., the editors that open when one double clicks an asset or selects "edit". Since
	// the editor is not a "primary" asset editor for any asset type, we do this management 
	// ourselves.
	// NOTE: If your asset class is associated with your editor, the asset editor subsystem can handle this
	for (TObjectPtr<UObject>& Object : ObjectsToEdit)
	{
		if (OpenedEditorInstances.Contains(Object))
		{
			OpenedEditorInstances[Object]->GetInstanceInterface()->FocusWindow(Object);
			return;
		}
	}

	// If we got here, there's not an instance already opened.
	UExampleCharacterFXEditor* CharacterFXEditor = NewObject<UExampleCharacterFXEditor>();

	// We should have done a check upstream to make sure that all of our targets are valid, but
	// we'll check again here.
	if (!ensure(AreObjectsValidTargets(ObjectsToEdit)))
	{
		return;
	}

	// Among other things, this call registers the editor with the asset editor subsystem,
	// which will prevent it from being garbage collected.
	CharacterFXEditor->Initialize(ObjectsToEdit);

	for (TObjectPtr<UObject>& Object : ObjectsToEdit)
	{
		OpenedEditorInstances.Add(Object, CharacterFXEditor);
	}
}

void UExampleCharacterFXEditorSubsystem::NotifyThatExampleCharacterFXEditorClosed(TArray<TObjectPtr<UObject>> ObjectsItWasEditing)
{
	for (TObjectPtr<UObject>& Object : ObjectsItWasEditing)
	{
		OpenedEditorInstances.Remove(Object);
	}
}
