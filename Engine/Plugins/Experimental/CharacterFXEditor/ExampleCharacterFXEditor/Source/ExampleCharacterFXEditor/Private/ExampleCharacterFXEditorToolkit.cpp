// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExampleCharacterFXEditorToolkit.h"
#include "ExampleCharacterFXEditorUISubsystem.h"
#include "ExampleCharacterFXEditorModule.h"
#include "ExampleCharacterFXEditorSubsystem.h"
#include "ExampleCharacterFXEditorMode.h"
#include "ExampleCharacterFXEditor.h"
#include "EditorModeManager.h"
#include "EditorViewportTabContent.h"
#include "Tools/UAssetEditor.h"

#define LOCTEXT_NAMESPACE "ExampleCharacterFXEditorToolkit"

FExampleCharacterFXEditorToolkit::FExampleCharacterFXEditorToolkit(UAssetEditor* InOwningAssetEditor)
	: FBaseCharacterFXEditorToolkit(InOwningAssetEditor, FName("ExampleCharacterFXEditor"))
{
	check(Cast<UExampleCharacterFXEditor>(InOwningAssetEditor));
}

FExampleCharacterFXEditorToolkit::~FExampleCharacterFXEditorToolkit()
{
	// We need to force the editor mode deletion now because otherwise the world
	// will end up getting destroyed before the mode's Exit() function gets to run, and we'll get some
	// warnings when we destroy any mode actors.
	EditorModeManager->DestroyMode(GetEditorModeId());

	// The editor subsystem is responsible for opening/focusing editor instances, so we should
	// notify it that this one is closing.
	// NOTE: Not necessary if your editor is the default editor for your asset
	UExampleCharacterFXEditorSubsystem* Subsystem = GEditor->GetEditorSubsystem<UExampleCharacterFXEditorSubsystem>();
	if (Subsystem)
	{
		TArray<TObjectPtr<UObject>> ObjectsWeWereEditing;
		OwningAssetEditor->GetObjectsToEdit(MutableView(ObjectsWeWereEditing));
		Subsystem->NotifyThatExampleCharacterFXEditorClosed(ObjectsWeWereEditing);
	}
}

FEditorModeID FExampleCharacterFXEditorToolkit::GetEditorModeId() const
{
	return UExampleCharacterFXEditorMode::EM_ExampleCharacterFXEditorModeId;
}

FText FExampleCharacterFXEditorToolkit::GetToolkitName() const
{
	const TArray<UObject*>* Objects = GetObjectsCurrentlyBeingEdited();
	if (Objects->Num() == 1)
	{
		return FText::Format(LOCTEXT("ExampleCharacterFXEditorTabNameWithObject", "ExampleCharacterFXEditor: {0}"),
			GetLabelForObject((*Objects)[0]));
	}
	return LOCTEXT("ExampleCharacterFXEditor", "ExampleCharacterFXEditor: Multiple");
}

FName FExampleCharacterFXEditorToolkit::GetToolkitFName() const
{
	return FName("ExampleCharacterFXEditor");
}

FText FExampleCharacterFXEditorToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("ExampleCharacterFXEditorToolkitName", "Example CharacterFX Editor");
}

#undef LOCTEXT_NAMESPACE
