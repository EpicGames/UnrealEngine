// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseCharacterFXEditor.h"
#include "ExampleCharacterFXEditor.generated.h"

UCLASS()
class UExampleCharacterFXEditor : public UBaseCharacterFXEditor
{
	GENERATED_BODY()
public:
	
	virtual TSharedPtr<FBaseAssetToolkit> CreateToolkit() override;
};

