// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseCharacterFXEditorCommands.h"

class FExampleCharacterFXEditorCommands : public TBaseCharacterFXEditorCommands<FExampleCharacterFXEditorCommands>
{
public:

	FExampleCharacterFXEditorCommands();

	// TInteractiveToolCommands<> interface
	virtual void RegisterCommands() override;
	virtual void GetToolDefaultObjectList(TArray<UInteractiveTool*>& ToolCDOs) override {}

public:

	TSharedPtr<FUICommandInfo> OpenCharacterFXEditor;

	const static FString BeginAttributeEditorToolIdentifier;
	TSharedPtr<FUICommandInfo> BeginAttributeEditorTool;
};
