// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseCharacterFXEditorMode.h"
#include "ExampleCharacterFXEditorMode.generated.h"

UCLASS()
class UExampleCharacterFXEditorMode : public UBaseCharacterFXEditorMode
{
	GENERATED_BODY()

public:

	const static FEditorModeID EM_ExampleCharacterFXEditorModeId;

	UExampleCharacterFXEditorMode();

	/**
	 * Gets the tool target requirements for the mode.
	 */
	static const FToolTargetTypeRequirements& GetToolTargetRequirements();

	virtual FBox SceneBoundingBox() const override;

protected:

	// UEdMode overrides
	virtual void CreateToolkit() override;
	virtual void BindCommands() override;
	virtual void Exit() override;

	// UBaseCharacterFXEditorMode overrides
	virtual void AddToolTargetFactories() override;
	virtual void RegisterTools() override;
	virtual void CreateToolTargets(const TArray<TObjectPtr<UObject>>& AssetsIn) override;

	virtual void InitializeTargets(const TArray<TObjectPtr<UObject>>& AssetsIn) override;

	// NOTE: Since the Example CharacterFX Editor is not the "default" editor for a particular asset type and we still want to have something to render,
	// we instead store a set of temporary DynamicMeshComponents here. This should not be necessary if your editor is the default for an asset type
	UPROPERTY()
	TArray<TObjectPtr<UDynamicMeshComponent>> DynamicMeshComponents;
};
