// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseCharacterFXEditorModeToolkit.h"

/**
 * The editor mode toolkit is responsible for the panel on the side in the editor
 * that shows mode and tool properties.
 */

class FExampleCharacterFXEditorModeToolkit : public FBaseCharacterFXEditorModeToolkit
{

public:

	/** For a specific tool palette category, construct and fill ToolbarBuilder with the category's tools **/
	virtual void BuildToolPalette(FName PaletteName, class FToolBarBuilder& ToolbarBuilder) override;

	// FBaseCharacterFXEditorModeToolkit
	virtual const FSlateBrush* GetActiveToolIcon(const FString& Identifier) const;

	// IToolkit
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;

};
