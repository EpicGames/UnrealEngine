// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseCharacterFXEditorModule.h"

class FLayoutExtender;

/**
 * NOTE: If the Editor is the default editor for an asset type then this class can be simplified. Much of the
 * code is here to enable opening the editor from the content browser for some specified asset types (e.g. SkeletalMesh, StaticMesh)
 * See also the note in ExampleCharacterFXEditorSubsystem.h
  */

class FExampleCharacterFXEditorModule : public FBaseCharacterFXEditorModule
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

protected:

	// NOTE: Only necessary because we want to allow opening specific asset types from the content browser
	void RegisterMenus();
};
