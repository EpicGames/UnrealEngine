// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"

/**
 * Slate style set for CharacterFX Editor
 */

class FExampleCharacterFXEditorStyle : public FSlateStyleSet
{
public:
	const static FName StyleName;

	/** Access the singleton instance for this style set */
	static FExampleCharacterFXEditorStyle& Get();

private:
	static FString InContent(const FString& RelativePath, const ANSICHAR* Extension);

	FExampleCharacterFXEditorStyle();
	~FExampleCharacterFXEditorStyle();
};

