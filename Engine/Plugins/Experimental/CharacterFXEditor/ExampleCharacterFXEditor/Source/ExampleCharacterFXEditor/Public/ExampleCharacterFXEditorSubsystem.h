// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSubsystem.h"
#include "ExampleCharacterFXEditorSubsystem.generated.h"

class UToolTargetManager;
class UToolTargetFactory;
class UExampleCharacterFXEditor;
class FToolTargetTypeRequirements;
class UToolTarget;

/*
 *  Typically an Editor's Subsystem would be where the actual Asset editing happens -- when the user presses a button in
 *  the Editor, the Subsystem would apply the change to the Asset. In this example editor, however, we are using
 *  the InteractiveToolsFramework for much of this machinery.
 * 
 *  This class still is useful if the Editor is not the default editor for a particular asset type. In this case,
 *  having an EditorSubsystem enables spawning the Editor for prescribed asset types (e.g. StaticMesh, SkeletalMesh).
 * 
 *  It's included here as an example, but if the concrete AssetEditor is associated with an asset type this class may not
 *  be necessary.
 */

UCLASS()
class UExampleCharacterFXEditorSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:

	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;

	/** Tries to build the core targets that provide meshes for tools to work on. */
	virtual void BuildTargets(const TArray<TObjectPtr<UObject>>& ObjectsIn,
		const FToolTargetTypeRequirements& TargetRequirements,
		TArray<TObjectPtr<UToolTarget>>& TargetsOut);

	/** Checks that all of the objects are valid targets for an editor session. */
	virtual bool AreObjectsValidTargets(const TArray<UObject*>& InObjects) const;

	/**
	 * Checks that all of the assets are valid targets for an editor session. This
	 * is preferable over AreObjectsValidTargets when we have FAssetData because it
	 * allows us to avoid forcing a load of the underlying UObjects (for instance to
	 * avoid triggering a load when right clicking an asset in the content browser).
	 */
	virtual bool AreAssetsValidTargets(const TArray<FAssetData>& InAssets) const;

	/**
	 * Either brings to the front an existing editor instance that is editing one of
	 * these objects, if one exists, or starts up a new instance editing all of these
	 * objects.
	 */
	virtual void StartExampleCharacterFXEditor(TArray<TObjectPtr<UObject>> ObjectsToEdit);

	/**
	 * Needs to be called when an editor instance is closed so that the subsystem knows
	 * to create a new one for these objects if they are opened again.
	 */
	virtual void NotifyThatExampleCharacterFXEditorClosed(TArray<TObjectPtr<UObject>> ObjectsItWasEditing);

protected:

	/**
	* Used to let the subsystem figure out whether targets are valid. New factories should be
	* added here in Initialize()/GetToolTargetRequirements() as they are developed.
	*/
	UPROPERTY()
	TObjectPtr<UToolTargetManager> ToolTargetManager = nullptr;

	// Manage instances of the editor
	TMap<TObjectPtr<UObject>, TObjectPtr<UExampleCharacterFXEditor>> OpenedEditorInstances;

	/** Create tool target factories based on what type of tool targets are supported */
	virtual void CreateToolTargetFactories(TArray<TObjectPtr<UToolTargetFactory>>& Factories) const;

};
