// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseCharacterFXEditorToolkit.h"

class FExampleCharacterFXEditorToolkit : public FBaseCharacterFXEditorToolkit
{
public:

	FExampleCharacterFXEditorToolkit(UAssetEditor* InOwningAssetEditor);
	virtual ~FExampleCharacterFXEditorToolkit();
	
	virtual FText GetToolkitName() const override;
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;

protected:

	// FBaseCharacterFXEditorToolkit overrides
	virtual FEditorModeID GetEditorModeId() const override;
};

