// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseCharacterFXEditorModeUILayer.h"
#include "ExampleCharacterFXEditorUISubsystem.generated.h"

UCLASS()
class UExampleCharacterFXEditorUISubsystem : public UBaseCharacterFXEditorUISubsystem
{
	GENERATED_BODY()

protected:

	virtual FName GetModuleName() const override
	{
		return FName("ExampleCharacterFXEditor");
	}
};
